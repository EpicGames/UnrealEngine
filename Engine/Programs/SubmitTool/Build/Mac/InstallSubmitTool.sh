P4VPath="SubmitTool"
if [ -e P4VId.txt ]; then
    P4VPath=`cat P4VId.txt`
fi

SubmitToolApp=$PWD/Mac/SubmitTool.app

echo "Registering $SubmitToolApp with P4V"

P4Xml=$HOME/Library/Preferences/com.perforce.p4v/customtools.xml
echo ""
echo "Updating $P4Xml..."
if [[ ! -e "$P4Xml" || -z "$(more "$P4Xml")" || -z "$(xmlstarlet sel -t -v "/CustomToolDefList" "$P4Xml")" ]]; then
    # Create a new basic customtools file
    echo "<?xml version=\"1.0\" encoding=\"UTF-8\"?>" > "$P4Xml"
    echo "<!--perforce-xml-version=1.0-->" >> "$P4Xml"
    echo "<CustomToolDefList varName=\"customtooldeflist\">" >> "$P4Xml"
    echo "</CustomToolDefList>" >> "$P4Xml"
else
    # Delete any existing tool with the same ID
    xmlstarlet ed --pf --inplace -d "/CustomToolDefList/CustomToolDef[Definition/Name='$P4VPath']" "$P4Xml"
fi

# Create a new tool entry
xmlstarlet ed --pf \
    -s "/CustomToolDefList" -t elem -n CustomToolDef \
    -s "/CustomToolDefList/CustomToolDef" -t elem -n Definition \
    -s '$prev' -t elem -n Name -v "$P4VPath" \
    -s '$prev/..' -t elem -n Command -v "$SubmitToolApp" \
    -s '$prev/..' -t elem -n Arguments -v '-n --args -server $p -user $u -client $c -root-dir \"$r\" -cl %p' \
    -s '$prev/../..' -t elem -n AddToContext -v 'true' \
    "$P4Xml" > "$P4Xml.tmp"

# make it look nice and write back into place
xmlstarlet fo -s 1 "$P4Xml.tmp" > "$P4Xml" 
rm "$P4Xml.tmp"