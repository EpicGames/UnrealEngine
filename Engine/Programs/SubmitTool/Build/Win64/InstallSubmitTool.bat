set /p MenuPath=<%~dp0P4VId.txt
IF "%MenuPath%" == "" set MenuPath="SubmitTool"
call "%~dp0P4vCustomInstaller/P4vCustomToolInstaller.exe" --toolpath "%~dp0/Windows/Engine/Binaries/Win64/SubmitTool.bat" --arguments "-server $p -user $u -client $c -root-dir \\\"$r\\\" -cl %%%%p" --menupath "%MenuPath%"