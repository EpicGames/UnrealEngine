# P4V custom tools file path
[string]$customtools = $env:USERPROFILE + "\.p4qt\customtools.xml"

Write-Host ""
Write-Host "Updating $customtools..."

# create a document if it does not exist
if (![System.IO.File]::Exists($customtools)) {
    $xml = [xml]'<?xml version="1.0" encoding="UTF-8"?>
    <!--perforce-xml-version=1.0-->
    <CustomToolDefList varName="customtooldeflist">
    </CustomToolDefList>
    '
    $xml.Save($customtools)
}

# get the submit tool path and entry 
[string]$path = Join-Path ($pwd).path "\Windows\Engine\Binaries\Win64\SubmitTool.bat"
[string]$entry = "SubmitTool"
[string]$p4vidfile = Join-Path ($pwd).path "P4VId.txt"
if ([System.IO.File]::Exists($p4vidfile)) {
    $entry = [System.IO.File]::ReadAllText($p4vidfile)
}

Write-Host "Registering $path with P4V as $entry"

# load the xml file
[xml]$xml = (Get-Content $customtools)

# remove existing node(s)
foreach($childnode in $xml.CustomToolDefList.SelectNodes("descendant::CustomToolDef[Definition/Name='$entry']")) {
    $xml.CustomToolDefList.RemoveChild($childnode) | Out-Null
}

# create a new node
[xml]$node =
@"
  <CustomToolDef>
    <Definition>
      <Name>$entry</Name>
      <Command>$path</Command>
      <Arguments>-n --args -server `$p -user `$u -client `$c -root-dir \"`$r\" -cl %p</Arguments>
    </Definition>
    <AddToContext>true</AddToContext>
    <Refresh>true</Refresh>
  </CustomToolDef>
"@

# import the new node and save the XML File
$xml.CustomToolDefList.AppendChild($xml.ImportNode($node.CustomToolDef, $true)) | Out-Null
$xml.Save($customtools)