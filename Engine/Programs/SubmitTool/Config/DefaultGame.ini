[Staging]
+RemapDirectories=(From="SubmitTool/Script", To="Engine/Binaries")
+AllowedDirectories=Engine/Content/Editor/Slate
+AllowedDirectories=Engine/Content/Slate
+AllowedConfigFiles=Engine/Config/BaseEditorPerProjectUserSettings.ini
+AllowedConfigFiles=SubmitTool/Config/SubmitTool.ini
+AllowedConfigFiles=SubmitTool/Config/SubmitToolDevOverride.ini
+AllowedConfigFiles=SubmitTool/Config/DeployId.ini
+AllowedConfigFiles=SubmitTool/Config/Mac/MacSubmitTool.ini
+AllowedConfigFiles=SubmitTool/Config/Linux/LinuxSubmitTool.ini
+AllowedConfigFiles=SubmitTool/Config/Windows/WindowsSubmitTool.ini

[/Script/UnrealEd.ProjectPackagingSettings]
+DirectoriesToAlwaysStageAsNonUFS=(Path="../Script")