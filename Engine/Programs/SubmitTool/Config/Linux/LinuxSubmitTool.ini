[Validator.NonUnityAndStaticAnalysis]
ExecutablePath="$(root)/Engine/Build/BatchFiles/Linux/Build.sh"
Platform="Linux"