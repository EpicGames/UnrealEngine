;To pack submit tool run these two commands substituting the correct paths
;RunUBT.bat -Target="SubmitTool Win64 Development -Target="UnrealPak Win64 Development -Project=Programs/SubmitTool/SubmitTool.uproject"
;RunUAT.bat BuildCookRun -platform=Win64 -project=SubmitTool -skipcook -stage -pak -nodebuginfo -nobootstrapexe -stagingdirectory=***STAGINGDESTINATION***/SubmitTool -NoCodeSign
;After these you will have a standalone submit tool you can zip and distribute in ***STAGINGDESTINATION***/SubmitTool

; For general parameters see Engine\Source\Programs\SubmitTool\Private\Parameters\SubmitToolParameters.h
[SubmitTool.CopyLog]
LogsToCollect=(\
	"$(SubmitToolSavedDir)Logs/SubmitTool.log"\
)

; Tags parameters (See Engine\Source\Programs\SubmitTool\Private\Models\TagDefinition.h):
;	Tags:
;		TagId -> tag to add in the CLDescription
;		TagLabel -> label to display in SubmitTool
;       ToolTip -> Tooltip to show on hovering the Tag
;       DocumentationUrl -> URL to navigate to upon clicking the tag help
;		InputType -> Type of widget {FreeText, PerforceUser}
;		ValueDelimiter -> string that acts as a delimiter to split different values
;		MinValues - theminimum amount of values that this tag should have
;		MaxValues -> the maximum amount of values this tag can have
;		bIsUserValue -> if true, values for this tag will need to be valid p4 usernames (if RegexValidation is present, both the regex and p4 users are valid) 
;       Filters -> if present acts as an allow list, any value that contains any of the listed filter will be included 
;       Validation -> How the tag is validated
;           IsMandatory -> if the tag needs to appear for the CL to be submitted
;           RegexValidation -> if present, validates the values of the tag against this regex
;           RegexErrorMessage -> Shows this error when RegexValidation fails
;       ValidationOverrides -> List of overrides for specific paths
;           RegexPath -> If any depot files matches this path use this validation instead:
;           Validation (see above)
;

[Tags.ReviewedBy]
TagId="#rb"
TagLabel="Reviewed By"
ToolTip="The RB (reviewed by) tag will display any reviewers on the change."
InputType="PerforceUser"
InputSubType="Swarm"
MinValues=1
MaxValues=10
bIsUserValue="true"
Validation=(\
    bIsMandatory="false",\
    RegexValidation="^skip$|^none$|^trivial$",\
    RegexErrorMessage="#RB tag value should be one of 'skip', 'none', 'trivial' or a valid p4 username or group"\
)
ValidationOverrides=(\
    (\
        RegexPath="^.*\.(?:c|cc|cpp|inl|m|mm|rc|cs|csproj|h|hpp|inl|usf|ush|uproject|uplugin|sln)$",\
        ConfigOverride=(\
            bIsMandatory="true",\
            RegexValidation="^skip$|^none$|^trivial$",\
            RegexErrorMessage="#RB tag value should be one of 'skip', 'none', 'trivial' or a valid p4 username or group, this tag is mandatory for code changes"\
        )\
    )\
)

[Tags.ComboSample]
TagId="#comboSample"
TagLabel="Combo"
ToolTip="This is an example."
DocumentationURL="https://www.google.com/"
InputType="MultiSelect"
MinValues=1
MaxValues=20
SelectValues=(\
    "I",\
    "am",\
    "an ",\
    "example")

[Tags.EmptySample]
TagId="#bool"
TagLabel="Bool"
ToolTip="Bool example."
InputType="Boolean"

[Tags.TextSample]
TagId="#text"
TagLabel="Value Tag"
ToolTip="Text example."
InputType="FreeText"

; Validator Parameters (more in Engine\Source\Programs\SubmitTool\Private\Logic\Validators\ValidatorDefinition.h)
;   [Validator.Id] -> The value after the . is the Validator unique ID
;	    Values:
;           Type                      -> Validator Class {TagValidator, CustomValidator, UBTValidator}
;           CustomName                -> Name of the custom validator, example <"MyCustomValidator">
;           IsRequired                -> True or False if this validator is required to pass to submit.  If it's not it will just throw warnings instead of blocking errors.
;           bRequireCompleteWhenOptional -> Whether the validator is required to finish running before allowing submission
;           TimeoutLimit              -> Timeout for validators, infinite if unspecified
;           
;           IncludeFilesWithExtension -> List extension for which this Validator needs to run.
;           IncludeFilesInDirectory   -> Only run this validator for files under this directory
;           ChangelistDescriptionAddendum -> This text will be added to the CL description if this validation passes
;           bSkipWhenAddendumInDescription -> Skip this validator when the addendum is already present in the CL description
;           SkipForbiddenTags         -> Skip is forbidden when this text if found in the CL description
;           DependsOn                 -> List of Validator Ids that needs to succeed before this validator runs
;           ExecutionBlockGroups      -> List of execution groups this Validator is part of. Two validators with an execution group in commmon cannot run concurrently
;           UIGroup                   -> Name of the UI Section this Validator is part of
;           bAcceptDeletedFiles       -> If this Validator runs on files marked for delete
;           bTreatWarningsAsErrors    -> If this Validator should treat warnings as errors
;           bUsesIncrementalCache     -> If this validator maintains a local cache of results per file between runs on the same CL, used for incremental validations
;           AdditionalValidationErrorMessages -> Additional error messages to print when this validation fails
;           ToolTip                   -> Tooltip when hovering over the Validator
;           TaskArea                  -> Area this validator works on, if an area is updated, the validator state will be automatically resetted { Everything, LocalFiles, ShelvedFiles, LocalAndShelvedFiles, Changelist }
;

[Validator.Tag]
Type="TagValidator"
CustomName="P4 Changelist Tag Validator"
IsRequired="true"
ToolTip="This validator ensures you have all of the required tags in your changelist description."
TaskArea="Changelist"

; This is an example of having a validator be a prerequisite for other validators.
; In this case UAT is often used by other validators. Compiling it once in this validator that other validators depend on
; can be more effiecent as they can be run in parallel with -NoCompileUAT -NoCompile -NoAutoSDK.
; IncludeFilesWithExtension should be a superset of all dependent filters.
; See the Validator.CheckBalancedMacros for how to depend on BuildUAT.
[Validator.BuildUAT]
Type="CustomValidator"
IsRequired="false"
bUsesIncrementalCache="true"
bLaunchHidden="true"
bLaunchReallyHidden="true"
CustomName="Build UAT"
IncludeFilesWithExtension=(".cc", ".cpp", ".cs", ".h", ".inl", ".m", ".mm", ".usf", ".ush", ".verse", ".versetest", "/Config/Tests/Tags.ini")
ExecutablePath="$(RunUAT)"
; For UE 5.7 and prior use the -WaitForUATMutex and -List arguments to compile everything
ExecutableArguments="-WaitForUATMutex -List"
; Post UE 5.7 (CL 47029854) use the -CompileOnly and -AutomationProject arguments
; ExecutableArguments="-WaitForUATMutex -CompileOnly -AutomationProject=AutomationScripts+BuildGraph"
ToolTip="This builds UAT once to let UAT based validators launch quicker and run in parallel."
ExecutionBlockGroups=("UBT")
TaskArea="LocalFiles"

[Validator.AnalyzeSharedDotNet]
Type="CustomValidator"
IsRequired="false"
bTreatWarningsAsErrors="true"
CustomName="Shared .NET Static Analysis"
bUsesIncrementalCache="true"
AppliesToCLRegex="Engine.Source.Programs.Shared"
ExecutablePath="$(engine)/Build/BatchFiles/RunDotnet$(ScriptExt)"
ExecutableArguments="build Source/Programs/Shared/Shared.sln -c Analyze -v minimal --no-incremental"
ToolTip="This validator builds the Shared .NET projects in the Analyze configuration to ensure that no static analysis issues have been introduced."
ExecutionBlockGroups=("DotNetAnalyze")
TaskArea="LocalFiles"

[Validator.NonUnityAndStaticAnalysis]
Type="UBTValidator"
IsRequired="true"
bUsesIncrementalCache="true"
CustomName="Non Unity + SA Validator"
IncludeFilesWithExtension=(".h", ".hpp", ".c", ".cpp")
ExecutablePath="$(root)/Engine/Build/BatchFiles/RunUBT.bat"
ExecutableArguments="-Define:CHECK_PUREVIRTUALS=1 -Define:UE_CHECK_DISABLE_OPTIMIZATION=1 -NoPCH -DisableUnity -AllModules -NoLink -WarningsAsErrors -IncludeHeaders"
StaticAnalysers=( "Default", "visualcpp", "clang", "PVSStudio")
StaticAnalyser="Default"
StaticAnalyserArg="-staticanalyzer="
bUseStaticAnalyser="true"
FileListArgument="-FileList="
TargetListArgument="-TargetList="
ProjectArgument="-Project="
Configuration="Development"
Platform="Win64"
Configurations=("Development", "Debug", "Shipping")
Platforms=("Win64", "Linux", "Mac")
Targets=("Editor", "Client")
ToolTip="This Validator compiles the code in the changelist using the configuration/platform provided, it also runs Static Code Analysis at the same time."
ErrorMessages=("A conflicting instance of", "dereferencing NULL pointer", "Unhandled exception:")
UIGroup="Code Validations"
ExecutionBlockGroups=("UBT")
TaskArea="LocalFiles"

[Validator.CheckBalancedMacros]
Type="CustomValidator"
IsRequired="false"
bUsesIncrementalCache="true"
bLaunchHidden="true"
bLaunchReallyHidden="true"
bTreatWarningsAsErrors="true"
CustomName="Balanced Macros Validator"
FileListArgument="-OverrideFileList="		
ExecutablePath="$(RunUAT)"
; The -NoCompileUAT -NoCompile -NoAutoSDK arguments are important as we expect UAT to already be compiled
ExecutableArguments="CheckBalancedMacros -NoCompileUAT -NoCompile -NoAutoSDK"
IncludeFilesWithExtension=(".h",".cpp")
ErrorMessages=("macro appears a second time without matching", "macro appears without matching", "macro does not have matching")
ToolTip="This validator checks for balanced macros. For example, if you have PRAGMA_DISABLE_OPTIMIZATION without a PRAGMA_ENABLE_OPTIMIZATION"
DependsOn=("BuildUAT")
TaskArea="LocalFiles"

[Validator.CrossChangelist]
Type="CrossChangelistValidator"
IsRequired="false"
Invalidated=("Files")
CustomName="Cross Changelists Validator"
bAcceptDeletedFiles="true"
ToolTip="This validator checks for both files in uncontrolled CLs and files that might need to be in the same changelist. A .h and .cpp in a different changelist might be an issue."
TaskArea="LocalFiles"

; This is an example of configuration for a validator that calls the editor with your project and runs a commandlet, to do in-editor validations
; You'll need to replace paths and commandlets, and tweak cmdline parameters
;
; [Validator.EditorCommandletExample]
; Type="CustomValidator"
; IsRequired="true"
; bLaunchHidden="true"
; bLaunchReallyHidden="false"
; CustomName="Editor Validator"
; bAcceptDeletedFiles="true"
; bTreatWarningsAsErrors="true"
; IncludeFilesWithExtension=(".uasset", ".umap", ".uplugin")
; ExecutableCandidates=(\
;     ("Editor Config: Development", "$(root)/Engine/Binaries/Win64/UnrealEditor.exe"),\
;     ("Editor Config: DebugGame", "$(root)/Engine/Binaries/Win64/UnrealEditor-Win64-DebugGame.exe"),\
;     ("Editor Config: Debug", "$(root)/Engine/Binaries/Win64/UnrealEditor-Win64-Debug.exe")\
; )
; bUseLatestExecutable="true"
; ExecutableArguments="********YOUR PROJECT NAME********* -Run=******YOUR COMMANDLET******* -P4Changelist=$(CL) -p4client=$(CLIENT) -abslog=$(SubmitToolLogsDir)/EditorValidation.log -unattended -traceautostart=0 -fastexit -ini:EditorPerProjectUserSettings:[/Script/LiveCoding.LiveCodingSettings]:bEnabled=false"
; IncludeFilesInDirectory="$(root)/********PATH TO YOUR PROJECT********"
; ChangelistDescriptionAddendum="#changelist validated"
; bSkipWhenAddendumInDescription="true"
; EnableOutputErrorsAnchor="Submit Tool Anchor - any error after this line shall be considered and cause error."
; DisableOutputErrorsAnchor="Submit Tool Anchor - any error after this line shall be ignored."
; ToolTip="This validator runs a commandlet in the local editor validation."
; UIGroup="Editor Validations"
; AdditionalValidationErrorMessages=("This validation runs UnrealEditor, please make sure you have a working non-crashing editor and, if you build your own, you have the right configuration selected in the validator options")
; ExecutionBlockGroups=("Editor")
; TaskArea="LocalFiles"
