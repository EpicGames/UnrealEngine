#!/bin/sh

zip=$1 shift
folder=$1 shift
name=$1 shift
version=$1 shift
versionfile=$1 shift
executable=$1 shift
rootdir=$1 shift

executableargs=""
for var in "$@"
do
    executableargs="$executableargs $var"
done

echo "--------------------------------------"
echo "ZIP: $zip"
echo "FOLDER: $folder"
echo "NAME: $name"
echo "VERSION: $version"
echo "VERSION FILE: $versionfile"
echo "EXE: $executable"
echo "ROOTDIR: $rootdir"
echo "ARGS: $executableargs"
echo "--------------------------------------"

echo "Waiting 2 seconds to allow $name to close"
sleep 2s

echo "Force closing all $name Instances..."
killall $name

# Not removing the folder to be safe and maintain parity with windows, re-enable if we find a reason
# echo "Deleting $folder"
#rm -rvf $folder

mkdir -p "$folder"

echo "Unzipping $folder"
unzip -o "$zip" -d "$folder"

rm -f "$versionfile"
versiondir=${versionfile%/*}
mkdir -p "$versiondir"
echo $version > $versionfile

echo "Invoking $executable with arguments: $executableargs"
nohup "$executable" $executableargs -root-dir "$rootdir" > /dev/null 2>&1 &

