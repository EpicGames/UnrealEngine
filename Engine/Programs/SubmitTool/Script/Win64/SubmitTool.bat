SET FULLPATH=%~dp0SubmitTool.exe
START /W "" "%FULLPATH%" %*