# Copyright Epic Games, Inc. All Rights Reserved.

$zip=$args[0]
$folder=$args[1]
$name=$args[2]
$version=$args[3]
$versionfile=$args[4]
$executable=$args[5]
$rootdir=$args[6]
$executableargs=$args[6..($args.Length-1)]

Write-Host "ZIP: $($zip)"
Write-Host "FOLDER: $($folder)"
Write-Host "NAME: $($name)"
Write-Host "VERSION: $($version)"
Write-Host "VERSION FILE: $($versionfile)"
Write-Host "EXE: $($executable)"
Write-Host "ROOTDIR: $($rootdir)"
Write-Host "ARGS: $($executableargs)"

Write-Host "Waiting 2 seconds to allow $($name) to close"
Start-Sleep -Seconds 2

Write-Host "Force closing all $($name) Instances..."
foreach ($process in Get-Process $name -ErrorAction:SilentlyContinue)
{
    Write-Host "Closing $($process.ProcessName) - $($process.Id)"
    Stop-Process -InputObject $process
    Get-Process | Where-Object {$_.HasExited} | Out-Null
}

Write-Host "Unzipping latest instance from $($zip) to $($folder)"
Expand-Archive $zip -DestinationPath $folder -Force

if (!(Test-Path $versionfile)) {
    New-Item -Path $versionfile
} else {
    Remove-Item -Path $versionfile
}
Add-Content $versionfile $version

Write-Host "Invoking $($executable) with arguments: $($executableargs)"
& "$executable" $executableargs -root-dir "$rootdir"
